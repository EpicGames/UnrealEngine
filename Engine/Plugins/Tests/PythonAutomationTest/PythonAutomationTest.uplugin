{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Python Automation Test",
	"Description": "",
	"Category": "Testing",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "PythonAutomationTest",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "PythonScriptPlugin",
			"Enabled": true
		},
		{
			"Name": "FunctionalTestingEditor",
			"Enabled": true
		},
	]
}