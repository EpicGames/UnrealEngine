// Copyright Epic Games, Inc. All Rights Reserved.

#include "PythonAutomationTest.h"
#include "Interfaces/IPluginManager.h"
#include "IPythonScriptPlugin.h"
#include "Misc/App.h"
#include "Misc/AutomationTest.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "PythonScriptTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PythonAutomationTest)

#define LOCTEXT_NAMESPACE "PyAutomationTest"

DEFINE_LOG_CATEGORY_STATIC(PyAutomationTest, Log, Log)

bool UPyAutomationTestLibrary::IsRunningPyLatentCommand = false;
float UPyAutomationTestLibrary::PyLatentCommandTimeout = 300.0f;

void UPyAutomationTestLibrary::SetIsRunningPyLatentCommand(bool isRunning)
{
	IsRunningPyLatentCommand = isRunning;
}

bool UPyAutomationTestLibrary::GetIsRunningPyLatentCommand()
{
	return IsRunningPyLatentCommand;
}

void UPyAutomationTestLibrary::SetPyLatentCommandTimeout(float Seconds)
{
	PyLatentCommandTimeout = Seconds;
}

float UPyAutomationTestLibrary::GetPyLatentCommandTimeout()
{
	return PyLatentCommandTimeout;
}

void UPyAutomationTestLibrary::ResetPyLatentCommand()
{
	IsRunningPyLatentCommand = false;
	PyLatentCommandTimeout = 300.0f;
}


void CleanUpPythonScheduler()
{
	// Clean up python side
	IPythonScriptPlugin::Get()->ExecPythonCommand(TEXT("import unreal;unreal.AutomationScheduler.cleanup()"));
}

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FIsRunningPyLatentCommand, float, Timeout);

bool FIsRunningPyLatentCommand::Update()
{
	float NewTime = FPlatformTime::Seconds();
	if (NewTime - StartTime < Timeout)
	{
		return !UPyAutomationTestLibrary::GetIsRunningPyLatentCommand();
	}

	UPyAutomationTestLibrary::SetIsRunningPyLatentCommand(false);
	FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest();
	const FString ErrMessage = FString::Printf(TEXT("Timeout reached waiting for Python Latent Command after %.2f sec."), Timeout);
	CurrentTest->AddError(ErrMessage);

	CleanUpPythonScheduler();

	return true;
}

class FPythonAutomationTestBase : public FAutomationTestBase
{
public:
	FPythonAutomationTestBase(const FString& InName, const bool bInComplexTask)
		: FAutomationTestBase(InName, bInComplexTask)
	{
	}

	virtual FString GetTestSourceFileName() const override { return __FILE__; }
	virtual FString GetTestSourceFileName(const FString& InTestName) const override
	{
		// Because FPythonAutomationTest is a Complex Automation Test, InTestName contains the name of the cpp class and a test parameter.
		// We isolate the test parameter and return it as it is the path of the python script.
		int Position = InTestName.Find(TEXT(" "));
		return InTestName.RightChop(Position+1);
	}

	virtual int32 GetTestSourceFileLine() const override { return __LINE__; }
	virtual int32 GetTestSourceFileLine(const FString& InTestName) const override
	{
		// FPythonAutomationTest generates one test per script file. File Line is therefore the begining of the file.
		return 0;
	}

protected:
	static FString BeautifyPath(FString Path)
	{
		int Position = Path.Find(TEXT("/Python/"), ESearchCase::CaseSensitive, ESearchDir::FromEnd);
		if (Position > 0) Position += 8;

		return Path.LeftChop(3).RightChop(Position).Replace(TEXT("/"), TEXT("."), ESearchCase::CaseSensitive);
	}

	virtual void SetTestContext(FString Context) override
	{
		TestParameterContext = BeautifyPath(Context);
	}

	/**
	 * Searches for python tests recursively based upon the given search path.
	 * The results are then placed in the OutBeatifiedNames and the python file name in OutFileNames
	 * This function is used to create additional UE automation tests by searching for python tests under the given directory in the given module
	 */
	static void SearchForPythonTests(const FString& ModuleName, const FString& ModuleSearchPath, TArray<FString>& OutBeautifiedNames, TArray<FString>& OutFileNames)
	{
		TArray<FString> FilesInDirectory;
		// only python files that start with 'test_' are considered a python test file.
		IFileManager::Get().FindFilesRecursive(FilesInDirectory, *ModuleSearchPath, TEXT("test_*.py"), true, false);

		// Scan all the found files, use only test_*.py file
		for (const FString& Filename : FilesInDirectory)
		{
			// Skip files from site-packages
			if (Filename.Contains(TEXT("/site-packages/")))
			{
				continue;
			}
			OutBeautifiedNames.Add(ModuleName + TEXT(".") + BeautifyPath(Filename));
			OutFileNames.Add(Filename);
		}
	}

private:
	FString PyTestName;
};

IMPLEMENT_CUSTOM_COMPLEX_AUTOMATION_TEST(
	FPythonAutomationTest,
	FPythonAutomationTestBase,
	"Editor.Python",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

void FPythonAutomationTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	// TODO - Scan also User folder Documents/UnrealEngine/Python or something define in Engine.ini (see FBX test builder)
	{
		// Find all files in the project dir under /Content/Python
		const FString PythonTestsDir = FPaths::Combine(
			FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir()),
			TEXT("Python"));
		
		SearchForPythonTests(FApp::GetProjectName(), PythonTestsDir, OutBeautifiedNames, OutTestCommands);
	}

	// Find all files under each plugin dir under /Content/Python
	TArray<TSharedRef<IPlugin>> EnabledPlugins = IPluginManager::Get().GetEnabledPlugins();
	for (const TSharedRef<IPlugin>& Plugin : EnabledPlugins)
	{
		const FString PluginContentDir = FPaths::Combine(
			FPaths::ConvertRelativePathToFull(Plugin->GetContentDir()),
			TEXT("Python"));

		SearchForPythonTests(Plugin->GetName(), PluginContentDir, OutBeautifiedNames, OutTestCommands);
	}
}

bool FPythonAutomationTest::RunTest(const FString& Parameters)
{
	bool Result = false;

	if (IPythonScriptPlugin::Get()->IsPythonAvailable())
	{
		UPyAutomationTestLibrary::ResetPyLatentCommand();
		CleanUpPythonScheduler();

		FPythonCommandEx PythonCommand;
		PythonCommand.Command = *FString::Printf(TEXT("\"%s\""), *Parameters); // Account for space in path
		PythonCommand.ExecutionMode = EPythonCommandExecutionMode::ExecuteFile;
		Result = IPythonScriptPlugin::Get()->ExecPythonCommandEx(PythonCommand);

		float Timout = UPyAutomationTestLibrary::GetPyLatentCommandTimeout();
		ADD_LATENT_AUTOMATION_COMMAND(FIsRunningPyLatentCommand(Timout));
	}
	else
	{
		AddError(TEXT("Python plugin is not available."));
	}

	return Result;
}

#undef LOCTEXT_NAMESPACE

