// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

class FPythonAutomationTestModule : public FDefaultModuleImpl {};

IMPLEMENT_MODULE(FPythonAutomationTestModule, PythonAutomationTest);
