// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "PythonAutomationTest.generated.h"

#define UE_API PYTHONAUTOMATIONTEST_API

UCLASS(MinimalAPI, meta = (ScriptName = "PyAutomationTest"))
class UPyAutomationTestLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	UFUNCTION(BlueprintCallable, Category = "PyAutomationTest")
	static UE_API void SetIsRunningPyLatentCommand(bool isRunning);

	UFUNCTION(BlueprintCallable, Category = "PyAutomationTest")
	static UE_API bool GetIsRunningPyLatentCommand();

	UFUNCTION(BlueprintCallable, Category = "PyAutomationTest")
	static UE_API void SetPyLatentCommandTimeout(float Seconds);

	UFUNCTION(BlueprintCallable, Category = "PyAutomationTest")
	static UE_API float GetPyLatentCommandTimeout();

	UFUNCTION(BlueprintCallable, Category = "PyAutomationTest")
	static UE_API void ResetPyLatentCommand();

private:
	static UE_API bool IsRunningPyLatentCommand;
	static UE_API float PyLatentCommandTimeout;
};

#undef UE_API
