// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PythonAutomationTest : ModuleRules
{
    public PythonAutomationTest(ReadOnlyTargetRules Target) : base(Target)
	{
        PublicDependencyModuleNames.AddRange
        (
            new string[] {
				"Core",
                "CoreUObject",
                "Engine",
				"Projects",
                "UnrealEd",
            }
        );
        
        PrivateDependencyModuleNames.AddRange(
             new string[] {
					"EditorFramework",
                    "PythonScriptPlugin",
                }
         );
    }
}
